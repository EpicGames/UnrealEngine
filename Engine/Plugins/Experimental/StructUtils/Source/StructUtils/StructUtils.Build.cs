// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class StructUtils : ModuleRules
	{
		public StructUtils(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

			PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
			}
			);

			// Code such as FInstancedStruct::NetSerialize relies on the engine, but if the engine
			// isn't available, this code will be compiled out using #if WITH_ENGINE.
			if (Target.bCompileAgainstEngine)
			{
				PrivateDependencyModuleNames.Add("Engine");
			}

			SetupIrisSupport(Target);
		}
	}
}
