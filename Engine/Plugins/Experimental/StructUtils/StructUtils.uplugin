{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Struct Utils",
	"Description": "Experimental Struct Utilities supplying InstancedStruct type",
	"Category": "Gameplay",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"DeprecatedEngineVersion": "5.5",
	"SupportedPrograms": [ "LiveLinkHub" ],
	"Modules": [
		{
			"Name": "StructUtils",
			"Type": "Runtime"
		},
		{
			"Name": "StructUtilsEngine",
			"Type": "Runtime"
		}
	],
	"Plugins": [
	]
}